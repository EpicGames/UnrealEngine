{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Light Mixer",
	"Description": "Edit any properties of scene lights in a spreadsheet format!",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": true,
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Hidden": true,
	"Plugins": [
		{
			"Name": "ObjectMixer",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "LightMixer",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
