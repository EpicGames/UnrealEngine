// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class LightMixer : ModuleRules
{
	public LightMixer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"OutputLog",
				"ObjectMixerEditor",
				"PropertyEditor"
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"AssetTools",
				"CoreUObject",
				"ContentBrowser",
				"Engine",
				"EditorStyle",
				"EditorWidgets",
				"InputCore",
				"Kismet",
				"Projects",
				"Slate",
				"SlateCore",
				"ToolMenus", 
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);
	}
}
