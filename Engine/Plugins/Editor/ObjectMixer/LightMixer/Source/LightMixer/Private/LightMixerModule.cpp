// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightMixerModule.h"

#include "LightMixerObjectFilter.h"
#include "LightMixerEditorSettings.h"
#include "LightMixerStyle.h"

#include "ISettingsModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FLightMixerEditorModule"

IMPLEMENT_MODULE(FLightMixerModule, LightMixer)

void FLightMixerModule::StartupModule()
{
	FLightMixerStyle::Initialize();
	
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FLightMixerModule::Initialize);
}

void FLightMixerModule::ShutdownModule()
{
	FLightMixerStyle::Shutdown();

	Teardown();
}

FLightMixerModule& FLightMixerModule::Get()
{
	return FModuleManager::LoadModuleChecked< FLightMixerModule >("LightMixer");
}

void FLightMixerModule::OpenProjectSettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings")
		.ShowViewer("Editor", "Plugins", "Light Mixer");
}

void FLightMixerModule::Initialize()
{
	FObjectMixerEditorModule::Initialize();
	
	DefaultFilterClass = ULightMixerObjectFilter::StaticClass();	
}

FName FLightMixerModule::GetModuleName() const
{
	return "LightMixer";
}

void FLightMixerModule::SetupMenuItemVariables()
{
	TabLabel = LOCTEXT("LightMixerTabLabel", "Light Mixer");

	MenuItemName = LOCTEXT("OpenLightMixerEditorMenuItem", "Light Mixer");
	MenuItemIcon =
		FSlateIcon(FLightMixerStyle::Get().GetStyleSetName(), "LightMixer.ToolbarButton", "LightMixer.ToolbarButton.Small");
	MenuItemTooltip = LOCTEXT("OpenLightMixerEditorTooltip", "Open Light Mixer");
}

FName FLightMixerModule::GetTabSpawnerId()
{
	return "LightMixerToolkit";
}

void FLightMixerModule::RegisterSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		// User Project Settings
		const TSharedPtr<ISettingsSection> ProjectSettingsSectionPtr = SettingsModule->RegisterSettings(
			"Editor", "Plugins", "Light Mixer",
			LOCTEXT("LightMixerSettingsDisplayName", "Light Mixer"),
			LOCTEXT("LightMixerSettingsDescription", "Configure Light Mixer user settings"),
			GetMutableDefault<ULightMixerEditorSettings>());
	}
}

void FLightMixerModule::UnregisterSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "Plugins", "Light Mixer");
	}
}

#undef LOCTEXT_NAMESPACE
