// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightMixerStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"
#include "Rendering/SlateRenderer.h"

TSharedPtr<FSlateStyleSet> FLightMixerStyle::StyleInstance = nullptr;

void FLightMixerStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FLightMixerStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

void FLightMixerStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FLightMixerStyle::Get()
{
	return *StyleInstance;
}

const FName& FLightMixerStyle::GetStyleSetName() const
{
	static FName StyleName(TEXT("LightMixer"));
	return StyleName;
}

const FSlateBrush* FLightMixerStyle::GetBrush(const FName PropertyName, const ANSICHAR* Specifier, const ISlateStyle* RequestingStyle) const
{
	return StyleInstance->GetBrush(PropertyName, Specifier);
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( Style->RootToContentDir(RelativePath, TEXT(".svg") ), __VA_ARGS__)
#define IMAGE_PLUGIN_BRUSH_SVG( PluginName, RelativePath, ... ) FSlateVectorImageBrush( FLightMixerEditorStyle::GetExternalPluginContent(PluginName, RelativePath, ".svg"), __VA_ARGS__)

const FVector2D Icon64x64(64.f, 64.f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon12x12(12.0f, 12.0f);
const FVector2D Icon8x8(8.f, 8.f);

FString FLightMixerStyle::GetExternalPluginContent(const FString& PluginName, const FString& RelativePath, const ANSICHAR* Extension)
{
	FString ContentDir = IPluginManager::Get().FindPlugin(PluginName)->GetBaseDir() / RelativePath + Extension;
	return ContentDir;
}

TSharedRef< FSlateStyleSet > FLightMixerStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>("LightMixer");
	
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("LightMixer"));
	if (ensure(Plugin.IsValid()))
	{
		Style->SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
	}

	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Icons
	Style->Set("LightMixer.ToolbarButton", new IMAGE_BRUSH_SVG("Icons/LightMixer", Icon40x40));
	Style->Set("LightMixer.ToolbarButton.Small", new IMAGE_BRUSH_SVG("Icons/LightMixer", Icon20x20));
	
	return Style;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
#undef IMAGE_BRUSH_SVG
#undef IMAGE_PLUGIN_BRUSH_SVG
