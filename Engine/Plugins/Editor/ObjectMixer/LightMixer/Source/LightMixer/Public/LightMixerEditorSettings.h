// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightMixerEditorSettings.generated.h"

UCLASS(MinimalAPI, config = ObjectMixer)
class ULightMixerEditorSettings : public UObject
{
	GENERATED_BODY()
public:
	
	ULightMixerEditorSettings(const FObjectInitializer& ObjectInitializer)
	{}

	/**
	 * If true, the Object Mixer menu item will be disabled and removed.
	 * This is useful if you only want to use Light Mixer and to avoid cluttering the menus.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Light Mixer", meta = (ConfigRestartRequired = true))
	bool bHideObjectMixerMenuItem = false;
};
