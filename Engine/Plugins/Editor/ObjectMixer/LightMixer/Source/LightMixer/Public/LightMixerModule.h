// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectMixerEditorModule.h"

#define UE_API LIGHTMIXER_API

class FLightMixerModule : public FObjectMixerEditorModule
{
public:

	//~ Begin IModuleInterface Interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	static UE_API FLightMixerModule& Get();
	
	static UE_API void OpenProjectSettings();
	
	//~ Begin FObjectMixerEditorModule overrides
	UE_API virtual void Initialize() override;
	UE_API virtual FName GetModuleName() const override;
	UE_API virtual void SetupMenuItemVariables() override;
	UE_API virtual FName GetTabSpawnerId() override;
	UE_API virtual void RegisterSettings() const override;
	UE_API virtual void UnregisterSettings() const override;
	//~ End FObjectMixerEditorModule overrides
};

#undef UE_API
