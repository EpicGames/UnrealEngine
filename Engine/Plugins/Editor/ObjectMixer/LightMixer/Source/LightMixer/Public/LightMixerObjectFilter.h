// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/LightComponent.h"
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"

#include "Engine/Light.h"

#include "LightMixerObjectFilter.generated.h"

UCLASS(MinimalAPI, BlueprintType, EditInlineNew)
class ULightMixerObjectFilter : public UObjectMixerObjectFilter
{
	GENERATED_BODY()
public:
	
	virtual TSet<UClass*> GetObjectClassesToFilter() const override
	{
		return
		{
			ULightComponent::StaticClass()
		};
	}

	virtual TSet<TSubclassOf<AActor>> GetObjectClassesToPlace() const override
	{
		return
		{
			ALight::StaticClass()
		};
	}

	virtual bool GetShowTransientObjects() const override
	{
		return true;
	}

	virtual TSet<FName> GetColumnsToShowByDefault() const override
	{
		return 
		{
			"Intensity", "LightColor", "LightingChannels", "AttenuationRadius"
		};
	}

	virtual TSet<FName> GetForceAddedColumns() const override
	{
		return {"LightColor", "LightingChannels"};
	}

	virtual bool ShouldIncludeUnsupportedProperties() const override
	{
		return false;
	}

	virtual EObjectMixerInheritanceInclusionOptions GetObjectMixerPropertyInheritanceInclusionOptions() const override
	{
		return EObjectMixerInheritanceInclusionOptions::IncludeAllParentsAndChildren;
	}

	virtual EObjectMixerInheritanceInclusionOptions GetObjectMixerPlacementClassInclusionOptions() const override
	{
		return EObjectMixerInheritanceInclusionOptions::IncludeAllChildren;
	}
};
